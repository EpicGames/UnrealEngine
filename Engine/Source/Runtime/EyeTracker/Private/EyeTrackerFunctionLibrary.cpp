// Copyright Epic Games, Inc. All Rights Reserved.

#include "EyeTrackerFunctionLibrary.h"
#include "Engine/Engine.h"
#include "IEyeTracker.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(EyeTrackerFunctionLibrary)

bool UEyeTrackerFunctionLibrary::IsEyeTrackerConnected()
{
	IEyeTracker const* const ET = GEngine ? GEngine->EyeTrackingDevice.Get() : nullptr;
	if (ET)
	{
		EEyeTrackerStatus const Status = ET->GetEyeTrackerStatus();
		return (Status != EEyeTrackerStatus::NotConnected);
	}

	return false;
}

bool UEyeTrackerFunctionLibrary::IsStereoGazeDataAvailable()
{
	IEyeTracker const* const ET = GEngine ? GEngine->EyeTrackingDevice.Get() : nullptr;
	if (ET)
	{
		return ET->IsStereoGazeDataAvailable();
	}

	return false;
}

bool UEyeTrackerFunctionLibrary::GetGazeData(FEyeTrackerGazeData& OutGazeData)
{
	IEyeTracker const* const ET = GEngine ? GEngine->EyeTrackingDevice.Get() : nullptr;
	if (ET)
	{
		return ET->GetEyeTrackerGazeData(OutGazeData);
	}
	
	OutGazeData = FEyeTrackerGazeData();
	return false;
}

bool UEyeTrackerFunctionLibrary::GetStereoGazeData(FEyeTrackerStereoGazeData& OutGazeData)
{
	IEyeTracker const* const ET = GEngine ? GEngine->EyeTrackingDevice.Get() : nullptr;
	if (ET)
	{
		return ET->GetEyeTrackerStereoGazeData(OutGazeData);
	}

	OutGazeData = FEyeTrackerStereoGazeData();
	return false;
}

void UEyeTrackerFunctionLibrary::SetEyeTrackedPlayer(APlayerController* PlayerController)
{
	IEyeTracker* const ET = GEngine ? GEngine->EyeTrackingDevice.Get() : nullptr;
	if (ET)
	{
		ET->SetEyeTrackedPlayer(PlayerController);
	}
}

