// Copyright Epic Games, Inc. All Rights Reserved.

#include "IEyeTrackerModule.h"
#include "Modules/ModuleManager.h"

FName IEyeTrackerModule::GetModularFeatureName()
{
	static const FName EyeTrackerModularFeatureName("EyeTracker");
	return EyeTrackerModularFeatureName;
}


class FEyeTrackerModule : public IEyeTrackerModule
{
	virtual TSharedPtr< class IEyeTracker, ESPMode::ThreadSafe > CreateEyeTracker()
	{
		TSharedPtr<IEyeTracker, ESPMode::ThreadSafe> DummyVal = nullptr;
		return DummyVal;
	}

	FString GetModuleKeyName() const
	{
		return FString("Default");
	}

	virtual bool IsEyeTrackerConnected() const override { return false; }
};

IMPLEMENT_MODULE(FEyeTrackerModule, EyeTracker);

