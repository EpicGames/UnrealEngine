// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "EyeTrackerTypes.h"
#include "IEyeTracker.h"
#include "EyeTrackerFunctionLibrary.generated.h"

#define UE_API EYETRACKER_API

UCLASS(MinimalAPI)
class UEyeTrackerFunctionLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	/**
	 * Returns whether or not the eye-tracking hardware is connected and ready to use. It may or may not actually be in use.
 	 * @return (Boolean)  true if eye tracker is connected and ready to use, false otherwise
	 */
	UFUNCTION(BlueprintPure, Category = "Eye Tracking")
	static UE_API bool IsEyeTrackerConnected();

	/**
	 * Returns whether or not the eye-tracking hardware is connected and ready to use. It may or may not actually be in use.
 	 * @return true if the connected eye tracker supports per-eye gaze data, false otherwise
	 */
	UFUNCTION(BlueprintPure, Category = "Eye Tracking")
	static UE_API bool IsStereoGazeDataAvailable();

	/**
	 * Returns unfied gaze data from the eye tracker. This is a single gaze ray, representing the fusion of both eyes.
	 * @param PlayerController		The player for whom we are tracking. Null can be ok for some devices, but this may be necessary for others to determine viewport properties, etc.
 	 * @return						True if the returned gaze data is valid, false otherwise. A false return is likely to be common (e.g. the when user blinks).
	 */
	UFUNCTION(BlueprintCallable, Category = "Eye Tracking")
	static UE_API bool GetGazeData(FEyeTrackerGazeData& OutGazeData);

	/**
	 * Returns stereo gaze data from the eye tracker. This includes a gaze ray per eye, as well as a fixation point.
	 * @param PlayerController		The player for whom we are tracking. Null can be ok for some devices, but this may be necessary for others to determine viewport properties, etc.
 	 * @return						True if the returned gaze data is valid, false otherwise. A false return is likely to be common (e.g. the when user blinks).
	 */
	UFUNCTION(BlueprintCallable, Category = "Eye Tracking")
	static UE_API bool GetStereoGazeData(FEyeTrackerStereoGazeData& OutGazeData);

	/**
	 * Specifies player being eye-tracked. This is not necessary for all devices, but is necessary for some to determine viewport properties, etc.
	 * @param PlayerController		The player for whom we are tracking. Null can be ok for some devices, but this may be necessary for others to determine viewport properties, etc.
	 */
	UFUNCTION(BlueprintCallable, Category = "Eye Tracking")
	static UE_API void SetEyeTrackedPlayer(APlayerController* PlayerController);

};

#undef UE_API
