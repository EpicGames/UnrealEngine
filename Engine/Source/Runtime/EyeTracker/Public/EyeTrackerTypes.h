// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Math/UnrealMathSSE.h"
#include "Math/Vector.h"
#include "UObject/ObjectMacros.h"

#include "EyeTrackerTypes.generated.h"


// struct FEyeTrackerKeys
// {
// 	static const FKey Gaze_X;
// 	static const FKey Gaze_Y;
// };

/** 
 * Represents a unified gaze ray that incorporates both eyes.
 */
USTRUCT(BlueprintType)
struct FEyeTrackerGazeData
{
	GENERATED_BODY()

public:

	FEyeTrackerGazeData()
		: GazeOrigin(ForceInitToZero)
		, GazeDirection(ForceInitToZero)
		, FixationPoint(ForceInitToZero)
		, ConfidenceValue(0.f)
		, bIsLeftEyeBlink(false)
		, bIsRightEyeBlink(false)
		, LeftPupilDiameter(0.0f)
		, RightPupilDiameter(0.0f)
	{}

	/** Origin of the unified gaze ray. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Gaze Data")
	FVector GazeOrigin;
	
	/** Forward direction of the unified gaze ray. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gaze Data")
	FVector GazeDirection;

	/** Location that the eyes converge. This is the 3d point where the tracked viewer is looking. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gaze Data")
	FVector FixationPoint;

	/** Value [0..1] that represents confidence in the gaze ray data above. Stable, reliably tracked eyes will be at or near 1. Fast-moving or unreliably tracked eyes will be at or near 0. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gaze Data")
	float ConfidenceValue;

	/** Left eye blink data. True if the eye is close. False if the eye is open.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gaze Data")
	bool bIsLeftEyeBlink;

	/** Right eye blink data. True if the eye is close. False if the eye is open. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gaze Data")
	bool bIsRightEyeBlink;

	/** Diameter of the left pupil. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gaze Data")
	float LeftPupilDiameter;

	/** Diameter of the right pupil */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Gaze Data")
	float RightPupilDiameter;
};


/** 
 * Stereo gaze data that contains information for each eye individually.
 * This may not be available with all devices.
 */
USTRUCT(BlueprintType)
struct FEyeTrackerStereoGazeData
{
	GENERATED_BODY()

public:

	FEyeTrackerStereoGazeData()
		: LeftEyeOrigin(ForceInitToZero)
		, LeftEyeDirection(ForceInitToZero)
		, RightEyeOrigin(ForceInitToZero)
		, RightEyeDirection(ForceInitToZero)
		, FixationPoint(ForceInitToZero)
		, ConfidenceValue(0.f)
	{}

	/** Origin of the left eye's gaze ray. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Stereo Gaze Data")
	FVector LeftEyeOrigin;
	/** Forward direction of the left eye's gaze ray. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Stereo Gaze Data")
	FVector LeftEyeDirection;

	/** Origin of the right eye's gaze ray. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Stereo Gaze Data")
	FVector RightEyeOrigin;
	/** Forward direction of the right eye's gaze ray. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Stereo Gaze Data")
	FVector RightEyeDirection;

	/** Location that the eyes converge. This is the 3d point where the tracked viewer is looking. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Stereo Gaze Data")
	FVector FixationPoint;

	/** Value [0..1] that represents confidence in the gaze ray data above. Stable, reliably tracked eyes will be at or near 1. Fast-moving or unreliably tracked eyes will be at or near 0. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Stereo Gaze Data")
	float ConfidenceValue;

};

UENUM(BlueprintType)
enum class EEyeTrackerStatus : uint8
{
	/** Eyetracking feature is not available (device not plugged in, etc) */
	NotConnected,
	/** Eyetracking is operating, but eyes are not being tracked */
	NotTracking,
	/** Eyetracking is operating and eyes are being tracked */
	Tracking,					
};
