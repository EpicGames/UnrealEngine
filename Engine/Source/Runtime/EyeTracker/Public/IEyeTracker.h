// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "InputCoreTypes.h"

#include "EyeTrackerTypes.h"

class IEyeTracker
{
public:

	/** 
	 * Specifies player being eye-tracked. This is not necessary for all devices, but is necessary for some to determine viewport properties, etc.. 
	 * Implementing class should cache this locally as it doesn't need to be called every tick.
	 */
	virtual void SetEyeTrackedPlayer(APlayerController* PlayerController) = 0;

	/** 
	 * Returns gaze data for the given player controller.
	 * @return true if returning valid data, false if gaze data was unavailable
	*/
	virtual bool GetEyeTrackerGazeData(FEyeTrackerGazeData& OutGazeData) const = 0;
	
	/** 
	 * Returns stereo gaze data for the given player controller (contains data for each eye individually).
	 * @return true if returning valid data, false if stereo gaze data was unavailable
	 */
	virtual bool GetEyeTrackerStereoGazeData(FEyeTrackerStereoGazeData& OutGazeData) const = 0;

	/** Returns information about the status of the current device. */
	virtual EEyeTrackerStatus GetEyeTrackerStatus() const = 0;

	/** Returns true if the current device can provide per-eye gaze data, false otherwise. */
	virtual bool IsStereoGazeDataAvailable() const = 0;
};
