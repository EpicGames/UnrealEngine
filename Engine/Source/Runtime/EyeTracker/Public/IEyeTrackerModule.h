// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Misc/ConfigCacheIni.h"
#include "Features/IModularFeatures.h"
#include "Features/IModularFeature.h"

/**
 * The public interface of the EyeTracker module
 */
class IEyeTrackerModule : public IModuleInterface, public IModularFeature
{
public:
	/** Returns modular feature name for this module */
	static EYETRACKER_API FName GetModularFeatureName();

	virtual FString GetModuleKeyName() const = 0;

	/** Returns the priority of this module from INI file configuration */
	// #todo jcf: need to work out prioritization and selection scheme
	// e.g. it doesn't ever really make sense to choose a desktop eye tracker if in VR mode?
	float GetModulePriority() const
	{
		float ModulePriority = 0.f;
		FString KeyName = GetModuleKeyName();
		GConfig->GetFloat(TEXT("EyeTrackerPluginPriority"), (!KeyName.IsEmpty() ? *KeyName : TEXT("Default")), ModulePriority, GEngineIni);
		return ModulePriority;
	}

	/** Sorting method for which plug-in should be given priority */
	struct FCompareModulePriority
	{
		bool operator()(IEyeTrackerModule& A, IEyeTrackerModule& B) const
		{
			return A.GetModulePriority() > B.GetModulePriority();
		}
	};

	/**
	 * Singleton-like access to IEyeTrackerModule
	 *
	 * @return Returns reference to the highest priority IEyeTrackerModule module
	 */
	static inline IEyeTrackerModule& Get()
	{
		TArray<IEyeTrackerModule*> ETModules = IModularFeatures::Get().GetModularFeatureImplementations<IEyeTrackerModule>(GetModularFeatureName());
		ETModules.Sort(FCompareModulePriority());
		return *ETModules[0];
	}

	/**
	 * Checks to see if there exists a module registered as an EyeTracker.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if there exists a module registered as an HMD.
	 */
	static inline bool IsAvailable()
	{
		return IModularFeatures::Get().IsModularFeatureAvailable(GetModularFeatureName());
	}

	/**
	* Register module as an EyeTracker on startup.
	*/
	virtual void StartupModule() override
	{
		IModularFeatures::Get().RegisterModularFeature(GetModularFeatureName(), this);
	}

	virtual bool IsEyeTrackerConnected() const = 0;

	/**
	 * Attempts to create a new head tracking device interface
	 *
	 * @return	Interface to the new head tracking device, if we were able to successfully create one
	 */
	virtual TSharedPtr< class IEyeTracker, ESPMode::ThreadSafe > CreateEyeTracker() = 0;
};
